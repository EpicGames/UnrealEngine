// Copyright Epic Games, Inc. All Rights Reserved.

#include "NormalMapIdentification.h"

#if WITH_EDITOR

#include "Engine/Texture2D.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define NORMALMAP_IDENTIFICATION_TIMING	(0)

#define LOCTEXT_NAMESPACE "NormalMapIdentification"

////////////////////////////////////////////////////////////////////////////////
// Constant values
namespace
{
	// These values may need tuning, but results so far have been good

	// These values are the threshold values for the average vector's
	// length to be considered within limits as a normal map normal
	const float NormalMapMinLengthConfidenceThreshold = 0.55f;
	const float NormalMapMaxLengthConfidenceThreshold = 1.1f;

	// This value is the threshold value for the average vector to be considered
	// to be going in the correct direction.
	const float NormalMapDeviationThreshold = 0.8f;

	// Samples from the texture will be taken in blocks of this size^2
	const int32 SampleTileEdgeLength = 4;

	// We sample up to this many tiles in each axis. Sampling more tiles
	// will likely be more accurate, but will take longer.
	const int32 MaxTilesPerAxis = 16;

	// This is used in the comparison with "mid-gray"
	const float ColorComponentNearlyZeroThreshold = (2.0f / 255.0f);

	// This is used when comparing alpha to zero to avoid picking up sprites
	const float AlphaComponentNearlyZeroThreshold = (1.0f / 255.0f);

	// These values are chosen to make the threshold colors (from uint8 textures)
	// discard the top most and bottom most two values, i.e. 0, 1, 254 and 255 on
	// the assumption that these are likely invalid values for a general normal map
	const float ColorComponentMinVectorThreshold = (2.0f / 255.0f) * 2.0f - 1.0f;
	const float ColorComponentMaxVectorThreshold = (253.0f/255.0f) * 2.0f - 1.0f;

	// This is the threshold delta length for a vector to be considered as a unit vector
	const float NormalVectorUnitLengthDeltaThreshold = 0.45f;

	// Rejected to taken sample ratio threshold.
	const float RejectedToTakenRatioThreshold = 0.33f;
}


////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////
class FNormalMapAnalyzer
{
public:
	FNormalMapAnalyzer(const FImageView& InMipToAnalyze)
	: NumSamplesTaken(0)
	, NumSamplesRejected(0)
	, NumSamplesThreshold(0)
	, AverageColor(0.0f,0.0f,0.0f,0.0f)
	, SourceImage(InMipToAnalyze)
	{
	}

	FVector3f ConvertToVectorForAnalysis(FLinearColor InColor)
	{
		// floating point formats don't need to be biased.
		FVector3f Result;
		if (SourceImage.Format == ERawImageFormat::BGRA8 ||
			SourceImage.Format == ERawImageFormat::RGBA16)
		{
			Result.X = InColor.R * 2.0f - 1.0f;
			Result.Y = InColor.G * 2.0f - 1.0f;
			Result.Z = InColor.B * 2.0f - 1.0f;
			return Result;
		}
		Result.X = InColor.R;
		Result.Y = InColor.G;
		Result.Z = InColor.B;
		return Result;
	}


	/**
	 * EvaluateSubBlock
	 * Iterates over all pixels in the specified rectangle, if the resulting pixel
	 * isn't black, mid grey or would result in X or Y being -1 or +1 then it is
	 * added to the average color and the number of samples count is incremented.
	 */
	void EvaluateSubBlock( int32 Left, int32 Top, int32 Width, int32 Height )
	{
		for ( int32 Y=Top; Y != (Top+Height); Y++ )
		{
			for ( int32 X=Left; X != (Left+Width); X++ )
			{
				FLinearColor ColorSample = SourceImage.GetOnePixelLinear( X, Y );

				// Nearly black or transparent pixels don't contribute to the calculation
				if (FMath::IsNearlyZero(ColorSample.A, AlphaComponentNearlyZeroThreshold) ||
					ColorSample.IsAlmostBlack())
				{
					continue;
				}

				// Scale and bias, if required, to get a signed vector
				FVector3f Vector = ConvertToVectorForAnalysis(ColorSample);
				const float Length = Vector.Length();
				if (Length < ColorComponentNearlyZeroThreshold)
				{
					// mid-grey pixels representing (0,0,0) are also not considered as they may be used to denote unused areas
					continue;
				}

				// If the vector is sufficiently different in length from a unit vector, consider it invalid.
				if (FMath::Abs(Length - 1.0f) > NormalVectorUnitLengthDeltaThreshold)
				{
					NumSamplesRejected++;
					continue;
				}

				// If the vector is pointing backwards then it is an invalid sample, so consider it invalid
				if (Vector.Z < 0.0f)
				{
					NumSamplesRejected++;
					continue;
				}

				AverageColor += ColorSample;
				NumSamplesTaken++;
			}
		}
	}

	/**
	 * DoesTextureLookLikelyToBeANormalMap
	 *
	 * Makes a best guess as to whether a texture represents a normal map or not.
	 * Will not be 100% accurate, but aims to be as good as it can without usage
	 * information or relying on naming conventions.
	 *
	 * The heuristic takes samples in small blocks across the texture (if the texture
	 * is large enough). The assumption is that if the texture represents a normal map
	 * then the average direction of the resulting vector should be somewhere near {0,0,1}.
	 * It samples in a number of blocks spread out to decrease the chance of hitting a
	 * single unused/blank area of texture, which could happen depending on uv layout.
	 *
	 * Any pixels that are black, mid-gray or have a red or green value resulting in X or Y
	 * being -1 or +1 are ignored on the grounds that they are invalid values. Artists
	 * sometimes fill the unused areas of normal maps with color being the {0,0,1} vector,
	 * but that cannot be relied on - those areas are often black or gray instead.
	 *
	 * If the heuristic manages to sample enough valid pixels, the threshold being based
	 * on the total number of samples it will be looking at, then it takes the average
	 * vector of all the sampled pixels and checks to see if the length and direction are
	 * within a specific tolerance. See the namespace at the top of the file for tolerance
	 * value specifications. If the vector satisfies those tolerances then the texture is
	 * considered to be a normal map.
	 */
	bool DoesTextureLookLikelyToBeANormalMap()
	{
		int32 TextureSizeX = SourceImage.GetWidth();
		int32 TextureSizeY = SourceImage.GetHeight();

		// Calculate the number of tiles in each axis, but limit the number
		// we interact with to a maximum of 16 tiles (4x4)
		int32 NumTilesX = FMath::Min( TextureSizeX / SampleTileEdgeLength, MaxTilesPerAxis );
		int32 NumTilesY = FMath::Min( TextureSizeY / SampleTileEdgeLength, MaxTilesPerAxis );

		if (( NumTilesX > 0 ) &&
			( NumTilesY > 0 ))
		{
			// If texture is large enough then take samples spread out across the image
			NumSamplesThreshold = (NumTilesX * NumTilesY) * 4; // on average 4 samples per tile need to be valid...

			for ( int32 TileY = 0; TileY < NumTilesY; TileY++ )
			{
				int Top = (TextureSizeY / NumTilesY) * TileY;

				for ( int32 TileX = 0; TileX < NumTilesX; TileX++ )
				{
					int Left = (TextureSizeX / NumTilesX) * TileX;

					EvaluateSubBlock( Left, Top, SampleTileEdgeLength, SampleTileEdgeLength );
				}
			}
		}
		else
		{
			NumSamplesThreshold = (TextureSizeX * TextureSizeY) / 4;

			// Texture is small enough to sample all texels
			EvaluateSubBlock( 0, 0, TextureSizeX, TextureSizeY );
		}

		// if we managed to take a reasonable number of samples then we can evaluate the result
		if ( NumSamplesTaken >= NumSamplesThreshold )
		{
			const float RejectedToTakenRatio = static_cast<float>(NumSamplesRejected) / static_cast<float>(NumSamplesTaken);
			if ( RejectedToTakenRatio >= RejectedToTakenRatioThreshold )
			{
				// Too many invalid samples, probably not a normal map
				return false;
			}

			AverageColor /= (float)NumSamplesTaken;

			// See if the resulting vector lies anywhere near the {0,0,1} vector
			FVector3f Vector = ConvertToVectorForAnalysis(AverageColor);

			float Magnitude = Vector.Length();

			// The normalized value of the Z component tells us how close to {0,0,1} the average vector is
			float NormalizedZ = Vector.Z / Magnitude;

			// if the average vector is longer than or equal to the min length, shorter than the max length
			// and the normalized Z value means that the vector is close enough to {0,0,1} then we consider
			// this a normal map
			return ((Magnitude >= NormalMapMinLengthConfidenceThreshold) &&
					(Magnitude < NormalMapMaxLengthConfidenceThreshold) &&
					(NormalizedZ >= NormalMapDeviationThreshold));
		}

		// Not enough samples, don't trust the result at all
		return false;
	}

	int32 NumSamplesTaken;
	int32 NumSamplesRejected;
	int32 NumSamplesThreshold;
	FLinearColor AverageColor;

	const FImageView& SourceImage;
};

/**
 * Attempts to evaluate the pixels in the texture to see if it is a normal map
 *
 * @param Texture The texture to examine
 *
 * @return bool true if the texture is likely a normal map (although it's not necessarily guaranteed)
 */
static bool IsImageANormalMap( const FImageView& Image, FStringView TextureDebugName )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IsImageANormalMap);

#if NORMALMAP_IDENTIFICATION_TIMING
	double StartSeconds = FPlatformTime::Seconds();
#endif

	// Analyze the source texture to try and figure out if it's a normal map.
	// First check is to make sure it's an appropriate surface format.
	
	bool bIsNormalMap = false;
	if (Image.Format == ERawImageFormat::BGRA8 ||
		Image.Format == ERawImageFormat::RGBA16 ||
		Image.Format == ERawImageFormat::RGBA16F ||
		Image.Format == ERawImageFormat::RGBA32F)
	{
		// The texture could be a normal map if it's one of these formats

		// for BGRA8 sources, interpret them as linear, not SRGB-encoded
		FImageView LinearImage = Image;
		if ( LinearImage.GetGammaSpace() != EGammaSpace::Linear )
		{
			// note, not converting pixels and copying image
			//	just reinterpretting
			LinearImage.GammaSpace = EGammaSpace::Linear;
		}

		FNormalMapAnalyzer Analyzer(LinearImage);
		bIsNormalMap = Analyzer.DoesTextureLookLikelyToBeANormalMap();
	}

#if NORMALMAP_IDENTIFICATION_TIMING
	double EndSeconds = FPlatformTime::Seconds();

	FString Msg = FString::Printf( TEXT("NormalMapIdentification took %f seconds to analyze %.*s"), (EndSeconds-StartSeconds), TextureDebugName.Len(), TextureDebugName.GetData() ); 

	GLog->Log(Msg);
#endif

	return bIsNormalMap;
}

/** Class to handle callbacks from notifications informing the user a texture was imported as a normal map */
class NormalMapImportNotificationHandler : public TSharedFromThis<NormalMapImportNotificationHandler>
{
public:
	NormalMapImportNotificationHandler() :
	  Texture(NULL)
	{
	}
	~NormalMapImportNotificationHandler()
	{
	}

	/** This method is invoked when the user clicks the "OK" button on the notification */
	void OKSetting(TSharedPtr<NormalMapImportNotificationHandler>)
	{
		if ( Notification.IsValid() )
		{
			Notification.Pin()->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
			Notification.Pin()->Fadeout();
		}
	}

	/* This method is invoked when the user clicked the "Revert" button on the notification */
	void RevertSetting(TSharedPtr<NormalMapImportNotificationHandler>)
	{
		UTexture2D* Texture2D = Texture.IsValid() ? Cast<UTexture2D>(Texture.Get()) : NULL;
		if ( Texture2D )
		{
			if ( Texture2D->CompressionSettings == TC_Normalmap )
			{
				Texture2D->PreEditChange(nullptr);
				Texture2D->SetFlags(RF_Transactional);

				Texture2D->CompressionSettings = TC_Default;
				Texture2D->LODGroup = TEXTUREGROUP_World;
				Texture2D->SRGB = true;

				Texture2D->PostEditChange();
			}
		}

		if ( Notification.IsValid() )
		{
			Notification.Pin()->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
			Notification.Pin()->Fadeout();
		}
	}

	TWeakObjectPtr<UTexture> Texture;
	TWeakPtr<SNotificationItem> Notification;
};

bool UE::NormalMapIdentification::HandleAssetPostImport( UTexture* Texture, const FImageView& InMipToAnalyze )
{
	if( Texture != NULL)
	{
		// Try to automatically identify a normal map
		//	this only reads Texture->Source
		if ( IsImageANormalMap( InMipToAnalyze, Texture->GetPathName()) )
		{
			// Set the compression settings and no gamma correction for a normal map
			{
				Texture->SetFlags(RF_Transactional);
				Texture->CompressionSettings = TC_Normalmap;
				Texture->SRGB = false;
				Texture->LODGroup = TEXTUREGROUP_WorldNormalMap;
			}

			// Show the user a notification indicating that this texture will be imported as a normal map.
			// Offer two options to the user, "OK" dismisses the notification early, "Revert" reverts the settings to that of a diffuse map.
			// ?? Guess?? this has to be done from main thread only??
			TSharedPtr<NormalMapImportNotificationHandler> NormalMapNotificationDelegate(new NormalMapImportNotificationHandler);
			{
				NormalMapNotificationDelegate->Texture = Texture;

				// this is a cheat to make sure the notification keeps the callback thing alive while it's active...
				FText OKText = LOCTEXT("ImportTexture_OKNormalMapSettings", "OK");
				FText OKTooltipText = LOCTEXT("ImportTexture_OKTooltip", "Accept normal map settings");
				FText RevertText = LOCTEXT("ImportTexture_RevertNormalMapSettings", "Revert");
				FText RevertTooltipText = LOCTEXT("ImportTexture_RevertTooltip", "Revert to diffuse map settings");

				FFormatNamedArguments Args;
				Args.Add( TEXT("TextureName"), FText::FromName(Texture->GetFName()) );
				FNotificationInfo NormalMapNotification( FText::Format(LOCTEXT("ImportTexture_IsNormalMap", "Texture {TextureName} was imported as a normal map"), Args ) );
				NormalMapNotification.ButtonDetails.Add(FNotificationButtonInfo(OKText, OKTooltipText, FSimpleDelegate::CreateSP(NormalMapNotificationDelegate.Get(), &NormalMapImportNotificationHandler::OKSetting, NormalMapNotificationDelegate)));
				NormalMapNotification.ButtonDetails.Add(FNotificationButtonInfo(RevertText, RevertTooltipText, FSimpleDelegate::CreateSP(NormalMapNotificationDelegate.Get(), &NormalMapImportNotificationHandler::RevertSetting, NormalMapNotificationDelegate)));
				NormalMapNotification.bFireAndForget = true;
				NormalMapNotification.bUseLargeFont = false;
				NormalMapNotification.bUseSuccessFailIcons = false;
				NormalMapNotification.bUseThrobber = false;
				NormalMapNotification.ExpireDuration = 10.0f;

				NormalMapNotificationDelegate->Notification = FSlateNotificationManager::Get().AddNotification(NormalMapNotification);
				if ( NormalMapNotificationDelegate->Notification.IsValid() )
				{
					NormalMapNotificationDelegate->Notification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
				}
			}

			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_EDITOR
