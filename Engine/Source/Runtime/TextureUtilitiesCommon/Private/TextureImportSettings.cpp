// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureImportSettings.h"
#include "HAL/IConsoleManager.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/ConfigCacheIni.h"
#include "ImageCoreUtils.h"
#include "Engine/Texture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureImportSettings)

UTextureImportSettings::UTextureImportSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Importing");
}

void UTextureImportSettings::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR
}

// Get the PNGInfill setting, with Default mapped to a concrete choice
ETextureImportPNGInfill UTextureImportSettings::GetPNGInfillMapDefault() const
{
	if ( PNGInfill == ETextureImportPNGInfill::Default )
	{
		// Default is OnlyOnBinaryTransparency unless changed by legacy config

		// get legacy config :
		bool bFillPNGZeroAlpha = true;
		if ( GConfig )
		{
			GConfig->GetBool(TEXT("TextureImporter"), TEXT("FillPNGZeroAlpha"), bFillPNGZeroAlpha, GEditorIni);		
		}
		
		return bFillPNGZeroAlpha ? ETextureImportPNGInfill::OnlyOnBinaryTransparency : ETextureImportPNGInfill::Never;
	}
	else
	{
		return PNGInfill;
	}
}

bool UTextureImportSettings::IsImportAutoVTEnabled() const
{
	if ( AutoVTSize <= 0 )
	{
		return false;
	}

	if ( ! UTexture::IsVirtualTexturingEnabled() )
	{
		return false;
	}

	static const auto CVarVirtualTexturesAutoImportEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.EnableAutoImport"));
	check(CVarVirtualTexturesAutoImportEnabled);

	return !! CVarVirtualTexturesAutoImportEnabled->GetValueOnAnyThread();
}

int64 UTextureImportSettings::GetAutoLimitPixelCount() const
{
	if ( AutoLimitDimension <= 0 )
	{
		// no limit
		return 0;
	}
	
	int64 UseAutoLimitDimension = AutoLimitDimension;

	UseAutoLimitDimension = FMath::Min<int64>(UseAutoLimitDimension, UTexture::GetMaximumDimensionOfNonVT());

	if ( IsImportAutoVTEnabled() )
	{
		if ( AutoVTSize != AutoLimitDimension )
		{
			// AutoVTSize and AutoLimitDimension cannot both be enabled and be different

			UE_CALL_ONCE( [&](){	
				UE_LOGF(LogCore, Warning, "VT is enabled with AutoVTSize (%d) not equal AutoLimitDimension (%d); they must be equal or zero, fix config!  Ignoring AutoLimitDimension and using AutoVTSize.", 
					AutoVTSize,AutoLimitDimension
					);	
			} );

			UseAutoLimitDimension = AutoVTSize;
		}
	}

	return UseAutoLimitDimension*UseAutoLimitDimension;
}

#if WITH_EDITOR
void UTextureImportSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
#endif // #if WITH_EDITOR



namespace UE::TextureUtilitiesCommon
{
	#if WITH_EDITOR
	/* Set default properties on Texture for newly imported textures, or reimports.
	*  Should be called after all texture properties are set, before PostEditChange() 
	*/
	TEXTUREUTILITIESCOMMON_API void ApplyDefaultsForNewlyImportedTextures(UTexture * Texture, bool bIsReimport)
	{
	
		// things that are done for both fresh import and reimport :

		if ( bIsReimport )
		{
			Texture->UpdateOodleTextureSdkVersionToLatest();

			// optional : at reimport ; if texture was previously nonpow2 but is now pow2,
			//		change some flags (UI, NeverStream, NoMipMaps, etc.) that may have been set by bDoAutomaticTextureSettingsForNonPow2Textures
			//		?this may stomp intentional user settings, but maybe that's just okay?
			//	ideally we'd like to just pop up a query box to say "yes/no" to that
			//	but we aren't allowed to do that here because we may be running on task threads in an Interchange mass-reimport
			
			if ( Texture->GetTextureClass() == ETextureClass::TwoD &&
				Texture->MipGenSettings == TMGS_NoMipmaps &&
				( Texture->CompressionSettings == TC_EditorIcon || Texture->CompressionSettings == TC_Default ) &&
				Texture->Source.IsValid() &&
				Texture->LODGroup != TEXTUREGROUP_UI )
			{
				// AreAllBlocksPowerOfTwo only checks XY but we are on TwoD class here only
				bool bIsPowerOfTwo = Texture->Source.AreAllBlocksPowerOfTwo() || ( Texture->PowerOfTwoMode != ETexturePowerOfTwoSetting::None );

				if ( bIsPowerOfTwo )
				{
					UE_LOGF(LogCore, Display, "Texture [%ls] settings changed by reimport; was NoMips UI, is now pow2 Default", 
						*Texture->GetName() );	

					// change to standard defaults for pow2 textures
					Texture->MipGenSettings = TMGS_FromTextureGroup;
					Texture->CompressionSettings = TC_Default;
					Texture->NeverStream = false;
				}
			}

			return;
		}

		// things that are for fresh import only :
		
		// SetModern does UpdateOodleTextureSdkVersionToLatest
		Texture->SetModernSettingsForNewOrChangedTexture();

		const UTextureImportSettings* Settings = GetDefault<UTextureImportSettings>();

		// cannot check for TC_Normalmap here
		//	because of the way NormalmapIdentification is delayed in Interchange
		//	it's harmless to just always turn it on
		//	it will be ignored if we are not TC_Normalmap
		//	OutBuildSettings.bNormalizeNormals = Texture.bNormalizeNormals && Texture.IsNormalMap();
		//if ( Texture->CompressionSettings == TC_Normalmap )
		{
			Texture->bNormalizeNormals = Settings->bEnableNormalizeNormals;
		}

		Texture->bUseNewMipFilter = Settings->bEnableFastMipFilter;

		// the pipeline before here will have set floating point textures to TC_HDR
		//	could alternatively check Texture->HasHDRSource
		if ( Texture->CompressionSettings == TC_HDR )
		{
			if ( Settings->CompressedFormatForFloatTextures == ETextureImportFloatingPointFormat::HDRCompressed_BC6 )
			{
				// use BC6H
				Texture->CompressionSettings = TC_HDR_Compressed;
			}
			else if ( Settings->CompressedFormatForFloatTextures == ETextureImportFloatingPointFormat::HDR_F32_or_F16 )
			{
				// set output format to match source format
				ETextureSourceFormat TSF = Texture->Source.GetFormat();
				if ( TSF == TSF_RGBA32F )
				{
					Texture->CompressionSettings = 	TC_HDR_F32;	
				}
				else if ( TSF == TSF_R32F )
				{
					Texture->CompressionSettings = 	TC_SingleFloat;
				}
				else if ( TSF == TSF_R16F )
				{
					Texture->CompressionSettings = 	TC_HalfFloat;
				}
				else
				{
					// else leave TC_HDR
					check( Texture->CompressionSettings == TC_HDR );
				}
			}
			else if ( Settings->CompressedFormatForFloatTextures == ETextureImportFloatingPointFormat::HDR_F16 )
			{
				// always use F16 HDR (legacy behavior)
				// leave TC_HDR
				check( Texture->CompressionSettings == TC_HDR );
			}
			else
			{
				// all cases should have been handled
				checkNoEntry();
			}
		}

		if ( Settings->bDoAutomaticTextureSettingsForNonPow2Textures )
		{
			if ( Texture->Source.GetNumBlocks() == 1 && ! Texture->Source.IsBlockPowerOfTwo(0) )
			{
				// try to set some better default options for non-pow2 textures

				// if Texture is not pow2 , change to TMGS_NoMipMaps (if it was default)
				//   this used to be done by Texture2d.cpp ; it is now optional
				//	 you can set it back to having mips if you want
				if ( Texture->MipGenSettings == TMGS_FromTextureGroup && Texture->GetTextureClass() == ETextureClass::TwoD )
				{
					Texture->MipGenSettings = TMGS_NoMipmaps;		
				}
		
				if ( ! Texture->Source.IsLongLatCubemap() )
				{
					// if Texture is not multiple of 4, change TC to EditorIcon ("UserInterface2D")
					//	if you do not do this, you might see "Texture forced to uncompressed because size is not a multiple of 4"
					//  this needs to match the logic in Texture.cpp : GetDefaultTextureFormatName
					int32 SizeX = Texture->Source.GetSizeX();
					int32 SizeY = Texture->Source.GetSizeY();
					if ( (SizeX&3) != 0 || (SizeY&3) != 0 )
					{
						if ( Texture->CompressionSettings == TC_Default ) // AutoDXT/BC1
						{
							Texture->CompressionSettings = TC_EditorIcon; // "UserInterface2D"
						}
					}
				}
			}
		}
	}
	
	bool ShouldTextureBeVirtualByAutoImportSize(const UTexture * Texture)
	{
		// If the texture is larger than a certain threshold make it VT.
		// Note that previously for re-imports we still checked size and potentially changed the VT status.
		// But that was unintuitive for many users so now for re-imports we will end up ignoring this and respecting the existing setting.
		
		if ( ! GetDefault<UTextureImportSettings>()->IsImportAutoVTEnabled() )
		{
			return false;
		}

		const int64 VirtualTextureAutoEnableThreshold = GetDefault<UTextureImportSettings>()->AutoVTSize;

		if ( VirtualTextureAutoEnableThreshold <= 0 )
		{
			return false;
		}
			
		const int64 VirtualTextureAutoEnableThresholdPixels = VirtualTextureAutoEnableThreshold * VirtualTextureAutoEnableThreshold;

		// We do this in pixels so a 8192 x 128 texture won't get VT enabled 
		// We use the Source size instead of simple Texture2D->GetSizeX() as this uses the size of the platform data
		// however for a new texture platform data may not be generated yet, and for an reimport of a texture this is the size of the
		// old texture. 
		// Using source size gives one small caveat. It looks at the size before mipmap power of two padding adjustment.
		// Textures with more than 1 block (UDIM textures) must be imported as VT
		if (Texture->Source.GetNumBlocks() > 1 ||
			( (int64) Texture->Source.GetSizeX() * Texture->Source.GetSizeY() ) >= VirtualTextureAutoEnableThresholdPixels ||
			Texture->Source.GetSizeX() > UTexture::GetMaximumDimensionOfNonVT() ||
			Texture->Source.GetSizeY() > UTexture::GetMaximumDimensionOfNonVT() )
		{
			return true;
		}

		return false;
	}
	#endif // WITH_EDITOR

	/* Get the default value for Texture->SRGB
	* ImportImageSRGB is the SRGB setting of the imported image
	*/
	TEXTUREUTILITIESCOMMON_API bool GetDefaultSRGB(TextureCompressionSettings TC, ETextureSourceFormat ImportImageFormat, bool ImportImageSRGB)
	{
		// Texture->SRGB sets the gamma correction of the platform texture we make
		//	so this is not just = ImportImageSRGB
		
		if ( TC == TC_Default || TC == TC_EditorIcon )
		{
			// DXT1,DXT3,R8G8B8 encodings
			//	we typically want SRGB on for these (for the platform encoding)
			// only exception is if the source is U8 linear
			//	 in that case, staying U8 linear probably preserves bits better
			//	 and ALSO we don't have the choice of turning on SRGB even if we want to
			//	 because of the way it is overloaded to mean both the source encoding and the platform encoding

			if ( ERawImageFormat::GetFormatNeedsGammaSpace( FImageCoreUtils::ConvertToRawImageFormat(ImportImageFormat) ) )
			{
				// if the imported image supported gamma (eg. U8)
				//	then we will set SRGB=false if it was U8-linear (can only happen with DDS U8 linear import, very rare)
				// note that texture SRGB flag in this case affects both the source interpretation and the platform encoding
				return ImportImageSRGB;
			}
			else
			{
				// counter-intuitively, U16 and F32 always want SRGB *on*
				//	the source will be treated as linear no matter what we set SRGB to
				//  SRGB will only affect the Platform encoding, so we prefer that to be sRGB color space
				return true;
			}
		}
		else
		{
			// TC_HDR, NormalMap, etc. we want SRGB off

			// TC_Grayscale we would prefer to have SRGB on, but default to off because
			//	G8 + SRGB is not supported well

			return false;
		}
	}

} // TextureUtilitiesCommon
