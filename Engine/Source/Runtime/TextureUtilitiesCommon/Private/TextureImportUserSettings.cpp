// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureImportUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureImportUserSettings)

UTextureImportUserSettings::UTextureImportUserSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SectionName = TEXT("Importing");
}

namespace UE::TextureUtilitiesCommon {
	TEXTUREUTILITIESCOMMON_API ETextureImportPNGInfill GetPNGInfillSetting()
	{
		// Try using per-user setting if present
		ETextureImportPNGInfill PNGInfill = GetDefault<UTextureImportUserSettings>()->PNGInfill;
		if (PNGInfill == ETextureImportPNGInfill::Default)
		{
			// If no user setting, use project setting if set, legacy settings/defaults if not.
			PNGInfill = GetDefault<UTextureImportSettings>()->GetPNGInfillMapDefault();
		}

		// "Default" should've been mapped to a concrete setting now.
		check(PNGInfill != ETextureImportPNGInfill::Default);
		return PNGInfill;
	}
}
