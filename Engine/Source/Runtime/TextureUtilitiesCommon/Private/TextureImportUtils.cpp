// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureImportUtils.h"

#include "ImageCore.h"
#include "Async/ParallelFor.h"
#include "Misc/MessageDialog.h"
#include "Engine/Texture.h"
#include "ImageCoreUtils.h"

namespace UE
{
	namespace TextureUtilitiesCommon
	{
		template<typename ImageClassType>
		bool AutoDetectAndChangeGrayScale(ImageClassType& Image)
		{
			if (Image.Format != ERawImageFormat::BGRA8)
			{
				return false;
			}

			// auto-detect gray BGRA8 and change to G8

			const FColor* Colors = (const FColor*)Image.RawData.GetData();
			int64 NumPixels = Image.GetNumPixels();

			for (int64 i = 0; i < NumPixels; i++)
			{
				if (Colors[i].A != 255 ||
					Colors[i].R != Colors[i].B ||
					Colors[i].G != Colors[i].B)
				{
					return false;
				}
			}

			// yes, it's gray, do it :
			Image.ChangeFormat(ERawImageFormat::G8, Image.GammaSpace);

			return true;
		}

		template TEXTUREUTILITIESCOMMON_API bool AutoDetectAndChangeGrayScale(FImage& Image);
		template TEXTUREUTILITIESCOMMON_API bool AutoDetectAndChangeGrayScale(FMipMapImage& Image);

		/**
		 * This fills any pixels of a texture with have an alpha value of zero and RGB=white,
		 * with an RGB from the nearest neighboring pixel which has non-zero alpha.

		   PNG images with "simple transparency" (eg. indexed color transparency) don't store RGB color in the transparent area
		   libpng decodes those pels are {RGB=white, A=0}
		   we replace them by filling in the RGB from neighbors

		   note that this does NOT fill in the RGB of PNGs with a full alpha channel. -> it does now, if PNGInfill == Always
		 */
		template<typename PixelDataType, typename ColorDataType, int32 RIdx, int32 GIdx, int32 BIdx, int32 AIdx>
		class TPNGDataFill
		{
		public:

			explicit TPNGDataFill( int32 SizeX, int32 SizeY, uint8* SourceTextureData )
				: SourceData( reinterpret_cast<PixelDataType*>(SourceTextureData) )
				, TextureWidth(SizeX)
				, TextureHeight(SizeY)
			{
				// libpng decodes simple transparent (binary A or indexed color) as { RGB=white, A = 0}

				if ( sizeof(ColorDataType) == 4 )
				{
					WhiteWithZeroAlpha = FColor(255, 255, 255, 0).DWColor();
				}
				else
				{
					uint16 RGBA[4] = { 0xFFFF,0xFFFF,0xFFFF, 0 };
					checkSlow( sizeof(ColorDataType) == 8 );
					checkSlow( sizeof(RGBA) == 8 );
					memcpy(&WhiteWithZeroAlpha,RGBA, sizeof(ColorDataType));
				}
				
				// falloff weights for near neighbor extrapolation
				// fast falloff -> just extend neighbor pels out
				// slow falloff -> blur neighbor pels together
				for(int r1=0;r1<=NearNeighborRadius;r1++)
				{
					for(int r2=0;r2<=NearNeighborRadius;r2++)
					{
						int rsqr = r1*r1 + r2*r2;
						if( rsqr == 0 )
						{
							NearNeighborWeights[0][0] = 0.f; // center self-weight = zero
							continue;
						}
						float W = expf( - 1.1f * sqrtf((float)rsqr) );
						NearNeighborWeights[r1][r2] = W;
					}
				}
			}

			void ProcessData(bool bDoOnComplexAlphaNotJustBinaryTransparency)
			{
				// first identify alpha type :
				bool HasWhiteWithZeroAlpha=false;
				bool HasComplexAlpha=false;
				for (int64 Y = 0; Y < TextureHeight; ++Y)
				{
					const ColorDataType* RowData = (const ColorDataType *)SourceData + Y * TextureWidth;

					for(int64 X = 0; X < TextureWidth; ++X)
					{
						if ( IsOpaque(RowData[X]) )
						{
						}
						else if ( RowData[X] == WhiteWithZeroAlpha )
						{
							HasWhiteWithZeroAlpha = true;
						}
						else
						{
							HasComplexAlpha = true;

							if ( ! bDoOnComplexAlphaNotJustBinaryTransparency )
							{
								UE_LOGF(LogCore, Log,  "PNG has complex alpha channel, will not fill RGB in transparent background, due to setting PNGInfill == OnlyOnBinaryTransparency");
								// do not modify png's with full alpha channels :
								return;
							}
						}
					}
				}

				if ( ! HasWhiteWithZeroAlpha )
				{
					// all opaque
					return;
				}

				if ( HasComplexAlpha )
				{
					UE_LOGF(LogCore, Log,  "PNG has alpha channel, doing fill of RGB in transparent background, due to setting PNGInfill == Always");
				}
				else
				{
					UE_LOGF(LogCore, Log,  "PNG has binary transparency, doing fill of RGB in transparent background, due to setting PNGInfill != Never");
				}

				// first do good fill with limited distance :
				// this ensures near pels within NearNeighborRadius get a good neighbor fill for interpolation

				FillFromNearNeighbors();

				// then do simple fill, rows one by one :
				// this can be a very poor fill, but it's fast for filling large empty areas

				// @todo oodle : fill from nearest row (up or down) rather than always filling downward

				int64 NumZeroedTopRowsToProcess = 0;
				int64 FillColorRow = -1;
				for (int64 Y = 0; Y < TextureHeight; ++Y)
				{
					if (!ProcessHorizontalRow(Y))
					{
						if (FillColorRow != -1)
						{
							FillRowColorPixels(FillColorRow, Y);
						}
						else
						{
							NumZeroedTopRowsToProcess = Y+1;
						}
					}
					else
					{
						FillColorRow = Y;
					}
				}

				// Can only fill upwards if image not fully zeroed
				if (NumZeroedTopRowsToProcess > 0 && NumZeroedTopRowsToProcess < TextureHeight)
				{
					for (int64 Y = 0; Y < NumZeroedTopRowsToProcess; ++Y)
					{
						// fill row at Y from row at NumZeroedTopRowsToProcess
						FillRowColorPixels(NumZeroedTopRowsToProcess, Y);
					}
				}
			}

			static bool IsOpaque(const ColorDataType InColor)
			{
				if constexpr (sizeof(ColorDataType) == 4)
				{
					return InColor >= 0xFF000000U;
				}
				else if constexpr (sizeof(ColorDataType) == 8)
				{
					return InColor >= 0xFFFF000000000000ULL;
				}
				else
				{
					static_assert(sizeof(ColorDataType) == 0);
					return false;
				}
			}

			static ColorDataType MakeColorWithZeroAlpha(const ColorDataType InColor)
			{
				// take the RGB from InColor
				// set A to zero
				// return that

				if ( sizeof(ColorDataType) == 4 )
				{
					return InColor & 0xFFFFFFU;
				}
				else if ( sizeof(ColorDataType) == 8 )
				{
					return InColor & 0xFFFFFFFFFFFFULL;
				}
				else
				{
					check(false);
				}
			}
			
			static ColorDataType MakeColorOpaque(const ColorDataType InColor)
			{
				// take the RGB from InColor
				// set A to opaque
				// return that

				if ( sizeof(ColorDataType) == 4 )
				{
					return InColor | 0xFF000000U;
				}
				else if ( sizeof(ColorDataType) == 8 )
				{
					return InColor | 0xFFFF000000000000ULL;
				}
				else
				{
					check(false);
				}
			}


			/* returns False if requires further processing because entire row is filled with zeroed alpha values */
			bool ProcessHorizontalRow(int64 Y)
			{		
				ColorDataType* RowData = (ColorDataType *)SourceData + Y * TextureWidth;
				int64 X = 0;

				// note this is done after the NN fill
				// the NN fill will have RGB != white but A = 0
				//	so we will fill out using those

				if ( RowData[0] == WhiteWithZeroAlpha )
				{
					// transparent run at start of row
					// find X which is the first opaque pel
					//	( "opaque" is a misnomer; actually transparent but not WhiteWithZeroAlpha )

					for(;;)
					{
						if ( RowData[X] != WhiteWithZeroAlpha )
						{
							break;
						}

						X++;

						if ( X == TextureWidth )
						{
							// whole row was transparent
							return false;
						}
					}

					check( X < TextureWidth );
					check( RowData[X] != WhiteWithZeroAlpha );

					// RowData[X] is opaque
					// fill initial run from it
					ColorDataType FillColor = MakeColorWithZeroAlpha(RowData[X]);
					for(int64 FillX=0;FillX<X;FillX++)
					{
						RowData[FillX] = FillColor;
					}
				}

				for(;;)
				{
					// we're in an opaque run, step to end of opaque run :
					check( RowData[X] != WhiteWithZeroAlpha );

					while( RowData[X] != WhiteWithZeroAlpha )
					{
						X++;

						if ( X == TextureWidth )
						{
							//reached end in opaque run, done.
							return true;
						}
					}

					// X is transparent
					check( X > 0 );
					check( RowData[X] == WhiteWithZeroAlpha );

					int64 FirstTransparent = X;
					while( RowData[X] == WhiteWithZeroAlpha )
					{
						X++;
						
						if ( X == TextureWidth )
						{
							//reached end in transparent run
							// fill right-only transparent run from left :
							
							ColorDataType FillColor = MakeColorWithZeroAlpha(RowData[FirstTransparent-1]);
							for(int64 FillX=FirstTransparent;FillX<TextureWidth;FillX++)
							{
								RowData[FillX] = FillColor;
							}

							return true;
						}
					}
					
					// RowData[X] is opaque
					check( X < TextureWidth );
					check( RowData[X] != WhiteWithZeroAlpha );
					// transparent run [FirstTransparent,X) with opaque on each side

					// fill left half from Left, right half from Right
					int64 MidX = (FirstTransparent + X)/2;
					ColorDataType FillLeft  = MakeColorWithZeroAlpha(RowData[FirstTransparent-1]);
					ColorDataType FillRight = MakeColorWithZeroAlpha(RowData[X]);
					
					for(int64 FillX=FirstTransparent;FillX<MidX;FillX++)
					{
						RowData[FillX] = FillLeft;
					}
					for(int64 FillX=MidX;FillX<X;FillX++)
					{
						RowData[FillX] = FillRight;
					}

					// X is opaque, repeat
				}
				// can't get here
			}

			void FillRowColorPixels(int64 FillColorRow, int64 Y)
			{
				// fill row Y from row FillColorRow , copying RGB only, leave A alone (A = 0 in dest)
				for (int64 X = 0; X < TextureWidth; ++X)
				{
					const PixelDataType* FillColor = SourceData + (FillColorRow * TextureWidth + X) * 4;
					PixelDataType* PixelData = SourceData + (Y * TextureWidth + X) * 4;
					PixelData[RIdx] = FillColor[RIdx];
					PixelData[GIdx] = FillColor[GIdx];
					PixelData[BIdx] = FillColor[BIdx];
				}
			}
			
			static FLinearColor MakeLinearColor(const ColorDataType InColor)
			{
				if ( sizeof(ColorDataType) == 4 )
				{
					// does SRGB gamma conversion :
					return FLinearColor( FColor(InColor) );
				}
				else
				{
					uint16 RGBA[4];
					checkSlow( sizeof(ColorDataType) == 8 );
					checkSlow( sizeof(RGBA) == 8 );
					memcpy(&RGBA,&InColor,sizeof(ColorDataType));

					return FLinearColor( 
						RGBA[0] * (1.f/0xFFFF),
						RGBA[1] * (1.f/0xFFFF),
						RGBA[2] * (1.f/0xFFFF),
						RGBA[3] * (1.f/0xFFFF));
				}
			}

			static ColorDataType MakeColorFromLinear(const FLinearColor InColor)
			{
				if ( sizeof(ColorDataType) == 4 )
				{
					return InColor.ToFColorSRGB().DWColor();
				}
				else
				{
					uint16 RGBA[4];
					checkSlow( sizeof(ColorDataType) == 8 );
					checkSlow( sizeof(RGBA) == 8 );

					RGBA[0] = FColor::QuantizeUNormFloatTo16( InColor.R );
					RGBA[1] = FColor::QuantizeUNormFloatTo16( InColor.G );
					RGBA[2] = FColor::QuantizeUNormFloatTo16( InColor.B );
					RGBA[3] = FColor::QuantizeUNormFloatTo16( InColor.A );

					ColorDataType Ret;
					memcpy(&Ret,RGBA,sizeof(ColorDataType));
					return Ret;
				}
			}

			ColorDataType GetFilledFromNearNeighbors(int64 CenterX,int64 CenterY) const
			{
				// look in the local neighborhood around CenterX,CenterY
				// find any opaque pixels that were in the source which we can expand from
				// (do not use opaque pixels made from previous NearNeighbor fills)
				// weight contribution by distance, falling off fast
				// (mainly we want the case of diagonal to average from +X and +Y)

				int64 LowX = FMath::Max(0,CenterX - NearNeighborRadius);
				int64 LowY = FMath::Max(0,CenterY - NearNeighborRadius);
				int64 HighX = FMath::Min(TextureWidth -1,CenterX + NearNeighborRadius);
				int64 HighY = FMath::Min(TextureHeight-1,CenterY + NearNeighborRadius);
				// search in [LowX,HighX] inclusive
				
				const ColorDataType* ImageColors = (const ColorDataType *)SourceData;
				float TotalWeight = 0.f;
				FLinearColor TotalColor(0,0,0,0);

				check( ImageColors[ CenterX + CenterY * TextureWidth ] == WhiteWithZeroAlpha );

				for(int64 Y=LowY;Y<=HighY;Y++)
				{
					const float * RowNearNeighborWeights = NearNeighborWeights[ FMath::Abs(Y - CenterY) ];

					for(int64 X=LowX;X<=HighX;X++)
					{
						const ColorDataType Color = ImageColors[ X + Y * TextureWidth ];

						#if 0
						// only fill from opaque colors
						if ( IsOpaque(Color) )
						{
							float W = RowNearNeighborWeights[ FMath::Abs(X-CenterX) ];
							// accumulate weighted color
							TotalWeight += W;
							TotalColor += W * MakeLinearColor(Color);
						}
						#else
						// fill from any non-transparent colors
						// weight by alpha
						FLinearColor CurColor = MakeLinearColor(Color);
						if ( CurColor.A != 0.f )
						{
							float W = RowNearNeighborWeights[ FMath::Abs(X-CenterX) ];

							// weighted by alpha :
							W *= CurColor.A;

							// accumulate weighted color
							TotalWeight += W;
							TotalColor += W * CurColor;
						}
						#endif

					}
				}

				if ( TotalWeight == 0.f )
				{
					// found nothing nearby
					return WhiteWithZeroAlpha;
				}

				TotalColor *= 1.f / TotalWeight;
				TotalColor.A = 0.f;

				// returned Color has A = 0 so it will not be read from in this pass
				//	if it has RGB != white it will not be filled again

				return MakeColorFromLinear(TotalColor);
			}

			void FillFromNearNeighbors()
			{
				// higher quality, slower neighbor fill with a limited distance of NearNeighborRadius

				ColorDataType* ImageColors = (ColorDataType *)SourceData;
								
				// this is trivially parallelizable because each pixel write op is independent
				//	and we don't read results that we write in this pass (because the A value excludes new writes)

				int32 NumRowsEachJob;
				int32 NumJobs = ImageParallelForComputeNumJobsForRows(NumRowsEachJob, TextureWidth, TextureHeight);

				ParallelFor(TEXT("TextureImportUtils.FillFromNearNeighbors.PF"), NumJobs, 1, [&](int32 Index)
				{
					int64 StartIndex = Index * NumRowsEachJob;
					int64 EndIndex = FMath::Min(StartIndex + NumRowsEachJob, TextureHeight);

					// using a scratch output row is not strictly required, but reduces cache thrashing between threads :
					//  (if single-threaded, no reason to use it, just write ImageRow[x] immediately)
					TArray<ColorDataType> ScratchRowArray;
					ScratchRowArray.SetNum(TextureWidth);
					ColorDataType * ScratchRow = &ScratchRowArray[0];

					for (int64 Y = StartIndex; Y < EndIndex; ++Y)
					{
						ColorDataType * ImageRow = ImageColors + Y * TextureWidth;
						for (int64 X = 0; X < TextureWidth; ++X)
						{
							//@todo Oodle : we could more quickly detect large areas where no fill within NearNeighborRadius is possible
							
							if ( ImageRow[X] == WhiteWithZeroAlpha )
							{
								// could write to ImageRow[X] immediately, but using ScratchRow reduces cache sharing across cores
								ScratchRow[X] = GetFilledFromNearNeighbors(X,Y);

								// ScratchRow[X] still has zero alpha, but no longer white
							}
							else
							{
								ScratchRow[X] = ImageRow[X];
							}
						}
						memcpy(ImageRow,ScratchRow,TextureWidth*sizeof(ColorDataType));
					}
				});
			}

			PixelDataType* SourceData;
			int64 TextureWidth;
			int64 TextureHeight;
			
			ColorDataType WhiteWithZeroAlpha;

			enum { NearNeighborRadius = 4 };
			float NearNeighborWeights[NearNeighborRadius+1][NearNeighborRadius+1];
		};

		void FillZeroAlphaPNGData(int32 SizeX, int32 SizeY, ETextureSourceFormat SourceFormat, uint8* SourceData, bool bDoOnComplexAlphaNotJustBinaryTransparency)
		{
			// These conditions should be checked by IsImportResolutionValid, but just in case we get here
			// via another path.
			check(SizeX > 0 && SizeY > 0);
			if (SizeX < 0 ||
				SizeY < 0)
			{
				return;
			}

			switch (SourceFormat)
			{
				case TSF_BGRA8:
				{
					TPNGDataFill<uint8, uint32, 2, 1, 0, 3> PNGFill(SizeX, SizeY, SourceData);
					PNGFill.ProcessData(bDoOnComplexAlphaNotJustBinaryTransparency);
					break;
				}

				case TSF_RGBA16:
				{
					TPNGDataFill<uint16, uint64, 0, 1, 2, 3> PNGFill(SizeX, SizeY, SourceData);
					PNGFill.ProcessData(bDoOnComplexAlphaNotJustBinaryTransparency);
					break;
				}

				default:
				{
					// G8, G16, no alpha to fill
					break;
				}
			}
		}
	}
}


bool UE::TextureUtilitiesCommon::IsImportResolutionValid(int64 Width, int64 Height, bool bAllowNonPowerOfTwo, FText* OutErrorMessage)
{
	// note: stricter than IsImageImportPossible
	//	IsImageImportPossible is a first check that can be done early in the loading to bail on totally impossible sizes
	//	this is done late and uses project-specific config

	// MaximumSupportedResolutionNonVT is only a popup/warning , not a hard limit
	// Get the non-VT size limit :
	int64 MaximumSupportedResolutionNonVT = (int64)UTexture::GetMaximumDimensionOfNonVT();

	// limit on current rendering RHI : == GetMax2DTextureDimension()
	//const int64 CurrentRHIMaxResolution = int64(1) << (GMaxTextureMipCount - 1);
	//MaximumSupportedResolutionNonVT = FMath::Min(MaximumSupportedResolutionNonVT, CurrentRHIMaxResolution);

	// No zero-size textures :
	if (Width == 0 || Height == 0)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = NSLOCTEXT("Interchange", "Warning_TextureSizeZero", "Texture has zero width or height");
		}

		return false;
	}

	// Dimensions must fit in signed int32
	//  could be negative here if it was over 2G and int32 was used earlier
	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = NSLOCTEXT("Interchange", "Warning_TextureSizeTooLargeOrInvalid", "Texture is too large to import or it has an invalid resolution.");
		}

		return false;
	}

	if (Width > MaximumSupportedResolutionNonVT || Height > MaximumSupportedResolutionNonVT)
	{
		if (! UTexture::IsVirtualTexturingEnabled() )
		{
			const FText VTMessage = NSLOCTEXT("Interchange", "Warning_LargeTextureVTDisabled", "\nWarning: Virtual Textures are disabled in this project.");

			if (EAppReturnType::Yes != FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, FText::Format(
				NSLOCTEXT("Interchange", "Warning_LargeTextureImport", "Attempting to import {0} x {1} texture, proceed?\nLargest supported non-VT texture size: {2} x {3}{4}"),
				FText::AsNumber(Width), FText::AsNumber(Height), FText::AsNumber(MaximumSupportedResolutionNonVT), FText::AsNumber(MaximumSupportedResolutionNonVT), VTMessage)))
			{
				return false;
			}
		}
	}

	// Check if the texture dimensions are powers of two
	if (!bAllowNonPowerOfTwo)
	{
		const bool bIsPowerOfTwo = FMath::IsPowerOfTwo(Width) && FMath::IsPowerOfTwo(Height);
		if (!bIsPowerOfTwo)
		{
			if ( OutErrorMessage )
			{
				*OutErrorMessage = NSLOCTEXT("Interchange", "Warning_TextureNotAPowerOfTwo", "Cannot import texture with non-power of two dimensions");
			}
			return false;
		}
	}

	return true;
}
