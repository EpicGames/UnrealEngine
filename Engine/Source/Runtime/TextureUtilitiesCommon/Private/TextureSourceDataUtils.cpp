// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureSourceDataUtils.h"

#if WITH_EDITOR

// TextureSourceDataUtils is *incorrectly* in /Runtime/
//	then the whole file is wrapped in WITH_EDITOR

#include "ImageCoreUtils.h"
#include "Engine/Texture.h"
#include "HAL/UnrealMemory.h"
#include "EngineLogs.h"
#include "TextureImportSettings.h"
#include "TextureBuildUtilities.h"
#include "ImageUtils.h"

namespace UE::TextureUtilitiesCommon::Experimental
{

namespace Private
{

	// adds wrap/clamp flags for X/Y on the filter as appropriate for this texture
	static FImageCore::EResizeImageFilter SetResizeImageFilterEdgeModes(UTexture * Texture,FImageCore::EResizeImageFilter Filter)
	{
		using namespace FImageCore;

		// default Filter with no flags will clamp at edges

		if ( Texture->GetTextureClass() == ETextureClass::Cube ||
			Texture->GetTextureClass() == ETextureClass::CubeArray )
		{
			if ( Texture->Source.IsLongLatCubemap() )
			{
				// wrap X, clamp Y
				return Filter | EResizeImageFilter::Flag_WrapX;
			}
			else
			{
				// clamp cube faces
				return Filter;
			}
		}
		else
		{
			// see ComputeAddressMode

			if ( Texture->bPreserveBorder )
			{
				// clamp
				return Filter;
			}

			// @@ nonpow2, UI, NoMipMaps -> clamp ?

			if ( Texture->GetTextureAddressX() == TA_Wrap )
			{
				Filter |= EResizeImageFilter::Flag_WrapX;
			}
			if ( Texture->GetTextureAddressY() == TA_Wrap )
			{
				Filter |= EResizeImageFilter::Flag_WrapY;
			}

			return Filter;
		}
	}

	// use ResizeImage instead of DownsizeImageUsingTextureSettings ?
	static bool UseResizeImageInsteadOfTextureSettings(UTexture * Texture)
	{
		if ( Texture->Source.IsLongLatCubemap() )
		{
			// longlat needs to wrap X and clamp Y which the TextureSettings resize can't do
			return true;
		}

		if ( Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::StretchToPowerOfTwo ||
			Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::StretchToSquarePowerOfTwo )
		{
			if ( ! Texture->Source.AreAllBlocksPowerOfTwo() )
			{
				// stretching required, ResizeImage instead
				return true;
			}
		}

		if ( Texture->MipGenSettings == TMGS_NoMipmaps ||
			Texture->MipGenSettings == TMGS_LeaveExistingMips ||
			Texture->MipGenSettings == TMGS_Angular )
		{
			// TextureSettings mip gen is ill defined in this case, don't use it
			return true;
		}

		return false;
	}

	// resize so that the largest dimension is <= MaxSize
	static bool ResizeTexture2D(UTexture* Texture, int32 MaxSize, const ITargetPlatform* TargetPlatform)
	{
		check( Texture->Source.GetNumLayers() == 1 );
		check( Texture->Source.GetNumBlocks() == 1 );
		const int32 LayerIndex = 0;

		// We want to reduce the asset size so ignore the imported mip(s) (??)
		const int32 MipIndex = 0;
		FImage Image;
		if (!Texture->Source.GetMipImage(Image, MipIndex))
		{
			UE_LOGF(LogTexture,Error,"ResizeTexture2D: Texture GetMipImage failed [%ls]",
				*Texture->GetFullName());
			return false;
		}

		if ( UseResizeImageInsteadOfTextureSettings(Texture) )
		{
			int32 TargetSizeX = Image.SizeX;
			int32 TargetSizeY = Image.SizeY;
			
			// stretch to pow2 like TextureCompressorModule
			if ( Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::StretchToPowerOfTwo )
			{
				TargetSizeX = FMath::RoundUpToPowerOfTwo(Image.SizeX);
				TargetSizeY = FMath::RoundUpToPowerOfTwo(Image.SizeY);				
			}
			else if ( Texture->PowerOfTwoMode == ETexturePowerOfTwoSetting::StretchToSquarePowerOfTwo )
			{
				TargetSizeX = TargetSizeY = FMath::RoundUpToPowerOfTwo( FMath::Max(Image.SizeX,Image.SizeY) );
			}

			// do halving steps to get <= MaxSize
			while( TargetSizeX > MaxSize || TargetSizeY > MaxSize )
			{
				TargetSizeX = FMath::Max(1,TargetSizeX>>1);
				TargetSizeY = FMath::Max(1,TargetSizeY>>1);
			}

			if ( TargetSizeX >= Image.SizeX && TargetSizeY >= Image.SizeY )
			{
				// nothing to do
				return false;
			}

			FImageCore::EResizeImageFilter Filter = SetResizeImageFilterEdgeModes(Texture,FImageCore::EResizeImageFilter::Default);

			FImageCore::ResizeImageInPlace(Image,TargetSizeX,TargetSizeY,Filter);
		}
		else
		{
			bool MadeChanges = false;
			if ( ! Texture->DownsizeImageUsingTextureSettings(TargetPlatform, Image, MaxSize, LayerIndex, MadeChanges) )
			{
				UE_LOGF(LogTexture,Error,"ResizeTexture2D: Texture DownsizeImageUsingTextureSettings failed [%ls]",
					*Texture->GetFullName());
				return false;
			}
			if ( ! MadeChanges )
			{
				return false;
			}
		}

		Texture->PreEditChange(nullptr);

		Texture->Source.Init(Image);

		// if gamma was Pow22 it is now sRGB
		Texture->bUseLegacyGamma = false;

		// PostEditChange is called outside by our caller

		return true;
	}

	static int32 RoundToNearestInt32PowerOfTwo(double X)
	{
		double Log2X = FMath::Log2(X);
		int32 IntLog2X = FMath::RoundToInt32(Log2X);
		
		return 1 << FMath::Clamp(IntLog2X,0,30);
	}

	// resize so that the largest dimension is == MaxSize
	static bool ResizeTextureToNearestPow2(UTexture* Texture)
	{
		check( Texture->Source.GetNumLayers() == 1 );
		check( Texture->Source.GetNumBlocks() == 1 );
		const int32 LayerIndex = 0;

		// discard imported mips :
		const int32 MipIndex = 0;

		FImage Image;
		if (!Texture->Source.GetMipImage(Image, MipIndex))
		{
			UE_LOGF(LogTexture,Error,"ResizeTexture2D: Texture GetMipImage failed [%ls]",
				*Texture->GetFullName());
			return false;
		}

		check( Image.SizeX == Texture->Source.GetSizeX() );
		check( Image.SizeY == Texture->Source.GetSizeY() );

		int32 TargetSizeX,TargetSizeY;

		// make the larger dimension go to nearest pow2 first
		// then fix the smaller dimension for aspect ratio
		if ( Image.SizeX >= Image.SizeY )
		{
			TargetSizeX = RoundToNearestInt32PowerOfTwo(Image.SizeX);
			TargetSizeY = RoundToNearestInt32PowerOfTwo( (double) TargetSizeX * Image.SizeY / Image.SizeX );
		}
		else
		{
			TargetSizeY = RoundToNearestInt32PowerOfTwo(Image.SizeY);
			TargetSizeX = RoundToNearestInt32PowerOfTwo( (double) TargetSizeY * Image.SizeX / Image.SizeY );
		}

		FImageCore::EResizeImageFilter Filter = SetResizeImageFilterEdgeModes(Texture,FImageCore::EResizeImageFilter::Default);

		FImageCore::ResizeImageInPlace(Image,TargetSizeX,TargetSizeY,Filter);
		
		Texture->PreEditChange(nullptr);

		Texture->Source.Init(Image);

		// if gamma was Pow22 it is now sRGB
		Texture->bUseLegacyGamma = false;

		// PostEditChange is called outside by our caller

		return true;
	}
	
	// concatenate all the image payloads into one bulkdata, eg. for mips or blocks
	static UE::Serialization::FEditorBulkData::FSharedBufferWithID MakeSharedBufferForImageDatas(const TArray<FImage> & InImages)
	{
		int64 SizeNeededInBytes = 0;
		for (const FImage& Im : InImages)
		{
			check( Im.RawData.Num() == Im.GetImageSizeBytes() );
			SizeNeededInBytes += Im.RawData.Num(); 
		}
		FUniqueBuffer WriteImageBuffer = FUniqueBuffer::Alloc(SizeNeededInBytes);

		uint8* CurrentAddress = static_cast<uint8*>(WriteImageBuffer.GetData());
		for (const FImage& Im : InImages)
		{
			FMemory::Memcpy(CurrentAddress, Im.RawData.GetData(), Im.RawData.Num());
			CurrentAddress += Im.RawData.Num();
		}

		return WriteImageBuffer.MoveToShared();
	}

	static bool ResizeTexture2DBlocked(UTexture* Texture, int32 TotalMaxSize, const ITargetPlatform* TargetPlatform)
	{
		// does not support layers
		check( Texture->Source.GetNumLayers() == 1 );
		const int32 NumLayers = 1;
		const int32 LayerIndex = 0;
		
		check( Texture->Source.GetNumBlocks() > 1 );

		// MaxSize is applied to the total UDIM size

		FIntPoint LogicalSourceSize = Texture->Source.GetLogicalSize();
		check( LogicalSourceSize.X > TotalMaxSize || LogicalSourceSize.Y > TotalMaxSize );

		double ResizeRatio = double(TotalMaxSize) / FMath::Max(LogicalSourceSize.X,LogicalSourceSize.Y);
		check( ResizeRatio < 1.0 );

		TArray<FTextureSourceBlock> ResizedSourceBlocks;
		ResizedSourceBlocks.Reserve(Texture->Source.GetNumBlocks());
	
		TArray<FImage> ResizedBlocks;
		ResizedBlocks.Reserve(Texture->Source.GetNumBlocks());

		bool MadeAnyChanges = false;

		for (int32 BlockIndex = 0; BlockIndex < Texture->Source.GetNumBlocks(); ++BlockIndex)
		{
			// We want to reduce the asset size so ignore the imported mip(s)
			FImage SourceMip0;
			const int32 MipIndex = 0;
			if (!Texture->Source.GetMipImage(SourceMip0, BlockIndex, LayerIndex, MipIndex))
			{
				UE_LOGF(LogTexture,Error,"ResizeTexture2DBlocked: Texture GetMipImage failed [%ls]",
					*Texture->GetFullName());
				return false;
			}

			int32 NewSizeX = FMath::RoundToInt32( ResizeRatio * SourceMip0.SizeX );
			int32 NewSizeY = FMath::RoundToInt32( ResizeRatio * SourceMip0.SizeY );
			int32 BlockMaxSize = FMath::Max(NewSizeX,NewSizeY);

			bool MadeChanges;
			if ( ! Texture->DownsizeImageUsingTextureSettings(TargetPlatform, SourceMip0, BlockMaxSize, LayerIndex, MadeChanges) )
			{
				// critical error
				UE_LOGF(LogTexture,Error,"ResizeTexture2DBlocked: Texture DownsizeImageUsingTextureSettings failed [%ls]",
					*Texture->GetFullName());
				return false;
			}
			MadeAnyChanges = MadeAnyChanges || MadeChanges;

			FTextureSourceBlock& ResizedSourceBlock = ResizedSourceBlocks.AddDefaulted_GetRef();
			Texture->Source.GetBlock(BlockIndex, ResizedSourceBlock);
		
			FImage& ResizedBlock = ResizedBlocks.AddDefaulted_GetRef();
			ResizedBlock = MoveTemp(SourceMip0);
			
			ResizedSourceBlock.SizeX = ResizedBlock.SizeX;
			ResizedSourceBlock.SizeY = ResizedBlock.SizeY;
			ResizedSourceBlock.NumMips = 1;
		}

		if ( ! MadeAnyChanges )
		{
			return false;
		}
		
		// Protect the code from an async build of the texture
		Texture->PreEditChange(nullptr);

		UE::Serialization::FEditorBulkData::FSharedBufferWithID ResizedImageBufferWithID = MakeSharedBufferForImageDatas(ResizedBlocks);

		const ETextureSourceFormat SourceFormat = Texture->Source.GetFormat();
		Texture->Source.InitBlocked(
			&SourceFormat, // array of formats per layer
			ResizedSourceBlocks.GetData(),
			NumLayers,
			ResizedSourceBlocks.Num(),
			MoveTemp(ResizedImageBufferWithID)
		);

		// if gamma was Pow22 it is now sRGB
		Texture->bUseLegacyGamma = false;

		return true;
	}

	static bool ResizeTextureToNearestPow2Blocked(UTexture* Texture)
	{
		// does not support layers
		check( Texture->Source.GetNumLayers() == 1 );
		const int32 NumLayers = 1;
		const int32 LayerIndex = 0;
		
		const int32 NumBlocks = Texture->Source.GetNumBlocks();
		check( NumBlocks > 1 );
	
		// Resize to Pow2 acts on each UDIM block, not the net size

		TArray<FTextureSourceBlock> SourceBlocks;
		SourceBlocks.SetNum(NumBlocks);
		
		int32 BlockMaxSizeX = 0;
		int32 BlockMaxSizeY = 0;

		// get the largest of all blocks (same as VT builder)
		for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			Texture->Source.GetBlock(BlockIndex, SourceBlocks[BlockIndex]);

			BlockMaxSizeX = FMath::Max(BlockMaxSizeX, SourceBlocks[BlockIndex].SizeX );
			BlockMaxSizeY = FMath::Max(BlockMaxSizeY, SourceBlocks[BlockIndex].SizeY );
		}
		
		int32 TargetSizeX,TargetSizeY;

		// make the larger dimension go to nearest pow2 first
		// then fix the smaller dimension for aspect ratio
		if ( BlockMaxSizeX >= BlockMaxSizeY )
		{
			TargetSizeX = RoundToNearestInt32PowerOfTwo(BlockMaxSizeX);
			TargetSizeY = RoundToNearestInt32PowerOfTwo( (double) TargetSizeX * BlockMaxSizeY / BlockMaxSizeX );
		}
		else
		{
			TargetSizeY = RoundToNearestInt32PowerOfTwo(BlockMaxSizeY);
			TargetSizeX = RoundToNearestInt32PowerOfTwo( (double) TargetSizeY * BlockMaxSizeX / BlockMaxSizeY );
		}

		TArray<FImage> ResizedBlocks;
		ResizedBlocks.SetNum(NumBlocks);

		for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			FImage & SourceMip0 = ResizedBlocks[BlockIndex];

			// Drops any imported mip data (yuck?)
			const int32 MipIndex = 0;
			if (!Texture->Source.GetMipImage(SourceMip0, BlockIndex, LayerIndex, MipIndex))
			{
				UE_LOGF(LogTexture,Error,"ResizeTexture2DBlocked: Texture GetMipImage failed [%ls]",
					*Texture->GetFullName());
				return false;
			}

			// all blocks have to be pow2 and the same aspect ratio
			// but they don't have to all be the same size
			// if they were different sizes, we could use a mip of TargetSize here; eg. (TargetSizeX>>1) 
			//	if that was closer to the original size
			// don't bother for now, just make them all the same size

			// if this block is == TargetSize, this is a nop
			FImageCore::ResizeImageInPlace(SourceMip0,TargetSizeX,TargetSizeY);
					
			SourceBlocks[BlockIndex].SizeX = TargetSizeX;
			SourceBlocks[BlockIndex].SizeY = TargetSizeY;
			SourceBlocks[BlockIndex].NumMips = 1;
		}
				
		// Protect the code from an async build of the texture
		Texture->PreEditChange(nullptr);

		UE::Serialization::FEditorBulkData::FSharedBufferWithID ResizedImageBufferWithID = MakeSharedBufferForImageDatas(ResizedBlocks);

		const ETextureSourceFormat SourceFormat = Texture->Source.GetFormat();
		Texture->Source.InitBlocked(
			&SourceFormat, // array of formats per layer
			SourceBlocks.GetData(),
			NumLayers,
			NumBlocks,
			MoveTemp(ResizedImageBufferWithID)
		);

		// if gamma was Pow22 it is now sRGB
		Texture->bUseLegacyGamma = false;

		return true;
	}

} // namespace Private

static bool IsTextureResizeSupported(UTexture * Texture)
{
	if ( ! ensure( Texture->Source.IsValid() ) )
	{
		return false;
	}

	// Check if we don't know how to resize that texture

	if ( Texture->Source.GetNumMips() > 1 && Texture->MipGenSettings == TMGS_LeaveExistingMips )
	{
		//	should not do this if MipGen == LeaveExisting ; or maybe warn?
		// return false;

		// go ahead and do it, but warn:

		UE_LOGF(LogTexture,Warning,"DownsizeTextureSourceData: Texture has LeaveExistingMips ; they will be discarded! [%ls]",
			*Texture->GetFullName());
	}

	// we only support 1 layer currently
	if (Texture->Source.GetNumLayers() != 1)
	{
		return false;
	}
	
	if (!(Texture->Source.GetTextureClass() == ETextureClass::Cube || Texture->Source.GetTextureClass() == ETextureClass::TwoD))
	{
		// array, cubearray, volume, not supported
		return false;
	}

	if (Texture->Source.GetTextureClass() == ETextureClass::Cube)
	{
		if (Texture->Source.IsLongLatCubemap())
		{
			if (Texture->Source.GetNumSlices() != 1)
			{
				return false;
			}
		}
		else if (Texture->Source.GetNumSlices() != 6)
		{
			return false;
		}
	}

	return true;
}
	
TEXTUREUTILITIESCOMMON_API bool ResizeTextureSourceDataToNearestPowerOfTwo(UTexture* Texture)
{
	if ( ! IsTextureResizeSupported(Texture) )
	{
		return false;
	}

	if ( Texture->Source.AreAllBlocksPowerOfTwo() )
	{
		return false;
	}
	
	if (Texture->Source.GetNumBlocks() == 1)
	{
		return Private::ResizeTextureToNearestPow2(Texture);
	}
	else
	{
		// UDIM VT
		return Private::ResizeTextureToNearestPow2Blocked(Texture);
	}
}

TEXTUREUTILITIESCOMMON_API bool DownsizeTextureSourceData(UTexture* Texture, int32 TargetSourceSize, const ITargetPlatform* TargetPlatform)
{
	if ( ! IsTextureResizeSupported(Texture) )
	{
		return false;
	}

	FIntPoint SourceSize = Texture->Source.GetLogicalSize();
	if (SourceSize.X <= TargetSourceSize && SourceSize.Y <= TargetSourceSize)
	{
		return false;
	}

	if (Texture->Source.GetNumBlocks() == 1)
	{
		return Private::ResizeTexture2D(Texture, TargetSourceSize, TargetPlatform);
	}
	else
	{
		// UDIM VT
		return Private::ResizeTexture2DBlocked(Texture, TargetSourceSize, TargetPlatform);
	}
}

TEXTUREUTILITIESCOMMON_API bool DownsizeTextureSourceDataNearRenderingSize(UTexture* Texture, const ITargetPlatform* TargetPlatform, int32 AdditionalSourceSizeLimit)
{
	if ( ! Texture->Source.IsValid() )
	{
		return false;
	}

	int32 BeforeSizeX;
	int32 BeforeSizeY;
	Texture->GetBuiltTextureSize(TargetPlatform, BeforeSizeX, BeforeSizeY);

	int32 TargetSizeInGame = FMath::Max(BeforeSizeX, BeforeSizeY);
	
	int32 TargetSourceSize = TargetSizeInGame;
	if (Texture->Source.IsLongLatCubemap())
	{
		// The function return the max size of the generated cube from the source long lat
		TargetSourceSize = UE::TextureBuildUtilities::ComputeLongLatCubemapExtents(TargetSizeInGame,0);
	}

	TargetSourceSize = FMath::Min(TargetSourceSize,AdditionalSourceSizeLimit);

	if (DownsizeTextureSourceData(Texture, TargetSourceSize, TargetPlatform))
	{
		Texture->LODBias = 0;
		
		// this counts as a reimport :
		UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,true);

		Texture->PostEditChange();
		
		// check that GetBuiltTextureSize was preserved :
		int32 AfterSizeX;
		int32 AfterSizeY;
		Texture->GetBuiltTextureSize(TargetPlatform, AfterSizeX, AfterSizeY);

		if ( BeforeSizeX != AfterSizeX ||
			 BeforeSizeY != AfterSizeY )
		{
			UE_LOGF(LogTexture,Warning,"DownsizeTextureSourceDataNearRenderingSize failed to preserve built size; was: %dx%d now: %dx%d on [%ls]",
				BeforeSizeX,BeforeSizeY,
				AfterSizeX,AfterSizeY,
				*Texture->GetFullName());
		}

		return true;
	}
	//	PreEditChange may have been called even if DownsizeTextureSourceData return false
	//	we don't PostEditChange here
	//	that's okay but not great

	return false;
}


TEXTUREUTILITIESCOMMON_API bool ChangeTextureSourceFormat(UTexture* Texture, ETextureSourceFormat NewFormat)
{
	if ( ! Texture->Source.IsValid() )
	{
		return false;
	}

	// we only support 1 layer currently
	if (Texture->Source.GetNumLayers() != 1)
	{
		return false;
	}
	
	ETextureSourceFormat OldFormat = Texture->Source.GetFormat(0);
	if ( OldFormat == NewFormat )
	{
		return false;
	}

	ERawImageFormat::Type NewRIF = FImageCoreUtils::ConvertToRawImageFormat(NewFormat);
	EGammaSpace NewGamma = ( Texture->SRGB && ERawImageFormat::GetFormatNeedsGammaSpace(NewRIF) ) ? EGammaSpace::sRGB : EGammaSpace::Linear;

	if ( Texture->Source.GetNumBlocks() == 1 && Texture->Source.GetNumMips() == 1 )
	{
		const int32 MipIndex = 0;

		FImage SourceMip;
		if (!Texture->Source.GetMipImage(SourceMip, MipIndex))
		{
			UE_LOGF(LogTexture,Error,"ChangeTextureSourceFormat: Texture GetMipImage failed [%ls]",
				*Texture->GetFullName());
			return false;
		}

		FImage NewMip;
		SourceMip.CopyTo(NewMip,NewRIF,NewGamma);
		
		Texture->PreEditChange(nullptr);

		Texture->Source.Init(NewMip);
	}
	else // blocks and/or mips
	{
		// all blocks of a UDIM have the same format; Layers do not
		int32 NumLayers = 1;
		int32 LayerIndex = 0;

		int32 NumBlocks = Texture->Source.GetNumBlocks();
		check( NumBlocks >= 1 );

		TArray<FTextureSourceBlock> NewBlocks;
		NewBlocks.Reserve(NumBlocks);
	
		TArray<FImage> NewImages;
		NewImages.Reserve(NumBlocks*16); // *16 for mips

		for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			FTextureSourceBlock Block;
			Texture->Source.GetBlock(BlockIndex,Block);

			// NewBlocks has sizes, they don't change
			NewBlocks.Add(Block);

			for(int32 MipIndex=0; MipIndex < Block.NumMips;MipIndex++)
			{
				FImage SourceMip;
				if ( ! Texture->Source.GetMipImage(SourceMip, BlockIndex, LayerIndex, MipIndex) )
				{
					UE_LOGF(LogTexture,Error,"ChangeTextureSourceFormat: Texture GetMipImage failed [%ls]",
						*Texture->GetFullName());

					return false;
				}
				
				FImage & NewMip = NewImages.AddDefaulted_GetRef();

				SourceMip.CopyTo(NewMip,NewRIF,NewGamma);
			}
		}

		UE::Serialization::FEditorBulkData::FSharedBufferWithID ResizedImageBufferWithID = Private::MakeSharedBufferForImageDatas(NewImages);
		
		Texture->PreEditChange(nullptr);

		Texture->Source.InitBlocked(
			&NewFormat, // array of formats per layer
			NewBlocks.GetData(),
			NumLayers,
			NewBlocks.Num(),
			MoveTemp(ResizedImageBufferWithID)
		);
	}

	// if gamma was Pow22 it is now sRGB
	Texture->bUseLegacyGamma = false;
	
	// this counts as a reimport :
	UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,true);

	Texture->PostEditChange();

	check( Texture->Source.GetGammaSpace(0) == NewGamma );

	return true;
}

// calls Pre/PostEditChange :
TEXTUREUTILITIESCOMMON_API bool CompressTextureSourceWithJPEG(UTexture* Texture,int32 Quality)
{
	if ( ! Texture->Source.IsValid() )
	{
		return false;
	}

	// we only support 1 layer currently
	if (Texture->Source.GetNumLayers() != 1)
	{
		return false;
	}
	
	if (Texture->Source.GetNumBlocks() != 1 )
	{
		// JPEG does not support UDIM/blocks ; fix me?
		return false;
	}
	
	if (Texture->Source.GetSourceCompression() == ETextureSourceCompressionFormat::TSCF_JPEG )
	{
		// already JPEG
		return false;
	}
	
	if ( Texture->Source.GetNumSlices() != 1 )
	{
		// 1 mip, 1 slice only
		return false;
	}

	ETextureSourceFormat Format = Texture->Source.GetFormat(0);
	if ( Format != TSF_G8 && Format != TSF_BGRA8 )
	{
		// must be 8 bit
		return false;
	}

	// we do kill existing mips to match the behavior of the other conversions in here

	// okay, looks good, do it!

	FImage Image;
	if ( ! Texture->Source.GetMipImage(Image,0) )
	{
		UE_LOGF(LogTexture,Error,"CompressTextureSourceWithJPEG: Texture GetMipImage failed [%ls]",
			*Texture->GetFullName());
		return false;
	}

	// JPEG it :

	TArray64<uint8> JPEGData;
	if ( ! FImageUtils::CompressImage(JPEGData,TEXT(".jpg"),Image,Quality) )
	{
		UE_LOGF(LogTexture,Error,"CompressTextureSourceWithJPEG: Texture CompressImage failed [%ls]",
			*Texture->GetFullName());
		return false;
	}

	Texture->PreEditChange(nullptr);

	// Format stays BGRA8 or G8
	const int32 NumMips = 1;
	Texture->Source.InitWithCompressedSourceData(Image.SizeX,Image.SizeY,NumMips,Format,
		JPEGData,
		ETextureSourceCompressionFormat::TSCF_JPEG);

	Texture->PostEditChange();
	
	return true;
}

} // End namespace UE::TextureUtilitiesCommon::Experimental

#endif //WITH_EDITOR
