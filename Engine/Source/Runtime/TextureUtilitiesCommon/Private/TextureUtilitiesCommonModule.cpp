// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

/**
 * A simple module that act as bridge between the Interchange plugin and the UnrealEd module
 * The goal of this module is to simply avoid some code duplication between the Interchange system and the old editor factories for the textures
 */

IMPLEMENT_MODULE(FDefaultModuleImpl, TextureUtilitiesCommon)
