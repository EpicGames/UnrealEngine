// Copyright Epic Games, Inc. All Rights Reserved.

#include "UDIMUtilities.h"

#include "Internationalization/Regex.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

namespace UE
{
	namespace TextureUtilitiesCommon
	{
		uint32 ParseUDIMName(const FString& Name, const FString& UdimRegexPattern, FString& OutPrefixName, FString& OutPostfixName)
		{
			FRegexPattern RegexPattern( UdimRegexPattern );
			FRegexMatcher RegexMatcher( RegexPattern, Name );

			int32 UdimValue = INDEX_NONE;

			if ( RegexMatcher.FindNext() )
			{
				const int32 StartOfCaptureGroup1 = RegexMatcher.GetCaptureGroupBeginning(1);
				const int32 EndOfCaptureGroup1 = RegexMatcher.GetCaptureGroupEnding(1);
				const int32 StartOfCaptureGroup2 = RegexMatcher.GetCaptureGroupBeginning(2);
				const int32 EndOfCaptureGroup2 = RegexMatcher.GetCaptureGroupEnding(2);
				const int32 StartOfCaptureGroup3 = RegexMatcher.GetCaptureGroupBeginning(3);
				const int32 EndOfCaptureGroup3 = RegexMatcher.GetCaptureGroupEnding(3);

				if ( StartOfCaptureGroup1 != INDEX_NONE && StartOfCaptureGroup2 != INDEX_NONE &&
					 EndOfCaptureGroup1 != INDEX_NONE && EndOfCaptureGroup2 != INDEX_NONE )
				{
					LexFromString( UdimValue, *Name.Mid( StartOfCaptureGroup2, EndOfCaptureGroup2 - StartOfCaptureGroup2 ) );

					OutPrefixName = Name.Mid( StartOfCaptureGroup1, EndOfCaptureGroup1 - StartOfCaptureGroup1 );

					if ( StartOfCaptureGroup3 != INDEX_NONE && EndOfCaptureGroup3 != INDEX_NONE )
					{
						OutPostfixName = Name.Mid( StartOfCaptureGroup3, EndOfCaptureGroup3 - StartOfCaptureGroup3 );
					}
				}
			}
	
			if ( UdimValue < UDIM_Min_Index || UdimValue > UDIM_Max_Index )
			{
				return INDEX_NONE;
			}

			return UdimValue;
		}

		int32 GetUDIMIndex(int32 BlockX, int32 BlockY)
		{
			check( BlockX >= 0 && BlockX <= UDIM_Max_X );
			check( BlockY >= 0 && BlockY <= UDIM_Max_Y );

			int32 Ret = BlockY * 10 + BlockX + 1001;

			check( Ret >= 1001 && Ret <= UDIM_Max_Index );

			return Ret;
		}

		TMap<int32, FString> GetUDIMBlocksFromSourceFile(const FString& File, const FString& UdimRegexPattern, FString* OutFilenameWithoutUdimPatternAndExtension)
		{
			FString PreUDIMNameToMatch;
			FString PostUDIMName;
			TMap<int32, FString> UDIMsAndSourcesFile;

			if (ParseUDIMName(FPaths::GetBaseFilename(File), UdimRegexPattern, PreUDIMNameToMatch, PostUDIMName) != INDEX_NONE)
			{ 
				const FString Path = FPaths::GetPath(File);
				const FString UDIMFilter = (Path / PreUDIMNameToMatch) + TEXT("*") + PostUDIMName + FPaths::GetExtension(File, true);

				TArray<FString> UDIMFiles;
				IFileManager::Get().FindFiles(UDIMFiles, *UDIMFilter, true, false);
				
				FString PreUDIMNameFound;
				for (FString& UDIMFile : UDIMFiles)
				{
					const int32 UDIMIndex = ParseUDIMName(FPaths::GetBaseFilename(UDIMFile), UdimRegexPattern, PreUDIMNameFound, PostUDIMName);
					
					if (UDIMIndex == INDEX_NONE || !PreUDIMNameFound.Equals(PreUDIMNameToMatch))
					{
						continue;
					}

					const FString UDIMName = PreUDIMNameFound + PostUDIMName;
					FString UDIMPath = FPaths::Combine(Path, UDIMFile);
					UDIMsAndSourcesFile.Add(UDIMIndex, MoveTemp(UDIMPath));
				}

				if (!UDIMsAndSourcesFile.IsEmpty() && OutFilenameWithoutUdimPatternAndExtension)
				{
					*OutFilenameWithoutUdimPatternAndExtension = PreUDIMNameToMatch + PostUDIMName;
				}
			}

			return UDIMsAndSourcesFile;
		}

		void ExtractUDIMCoordinates(int32 UDIMIndex, int32& OutBlockX, int32& OutBlockY)
		{
			check( UDIMIndex != INDEX_NONE );
			check( UDIMIndex >= UDIM_Min_Index && UDIMIndex <= UDIM_Max_Index );

			// note 1010 is X=9, Y=0 , that is the tens column is not Y
			OutBlockX = (UDIMIndex - UDIM_Min_Index) % 10;
			OutBlockY = (UDIMIndex - UDIM_Min_Index) / 10;

			check( OutBlockX >= 0 && OutBlockX <= UDIM_Max_X );
			check( OutBlockY >= 0 && OutBlockY <= UDIM_Max_Y );
		}
	}
}
