// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ImageCore.h"

class UTexture;
class UTextureFactory;

namespace UE::NormalMapIdentification
{
	/**
	 * Checks if the given mip view of the given texture looks like if contains normal map data, and if so
	 * updates the texture compression settings to be for a normal map. Additionally fires off a UI
	 * notification with a revert option, as this is a guess and could very well be wrong.
	 * 
	 * This does not examine the entire mip but instead checks small chunks across the mip.
	 * 
	 * This should be called when texture properties are safe to edit (i.e. between Pre/PostEditChange).
	 * 
	 * @return true if the asset was identified as normal
	 */
	bool TEXTUREUTILITIESCOMMON_API HandleAssetPostImport( UTexture* InTexture, const FImageView& InMipToAnalyze );
}

#endif ///WITH_EDITOR
