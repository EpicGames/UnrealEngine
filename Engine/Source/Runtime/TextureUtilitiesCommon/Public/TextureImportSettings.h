// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/TextureDefines.h"

#include "TextureImportSettings.generated.h"

struct FPropertyChangedEvent;

/* What CompressionSettings runtime format should imported floating point textures use
 */
UENUM()
enum class ETextureImportFloatingPointFormat : uint8
{
	/* Use "HDR" RGBA16F */
	HDR_F16 = 0,
	/* Use "HDRCompressed" , BC6H */
	HDRCompressed_BC6,
	/* Use 32-bit float formats if the source is 32-bit, otherwise use 16-bit HDR */
	HDR_F32_or_F16,

	PreviousDefault = HDR_F16  UMETA(Hidden) // legacy behavior
};


/* When should RGB colors be spread into neighboring fully transparent white pixels, replacing their RGB.
   By default, this is done OnlyOnBinaryTransparency, not on PNG's with non-binary-transparency alpha channels.
   The PNG format has two different ways of storing alpha, either as 1-bit binary transparency, or as full 8/16 bit alpha channels.

   Used to be set from the TextureImporter/FillPNGZeroAlpha config value.  Setting this option will supersede that.
 */
UENUM()
enum class ETextureImportPNGInfill : uint8
{
	/* Use the default behavior. For user settings, this means use project settings. For project settings, it's set from the TextureImporter/FillPNGZeroAlpha config value; default was OnlyOnBinaryTransparency. */
	Default = 0,
	/* Never infill RGB, import the PNG exactly as it is stored in the file. */
	Never,
	/* Only infill RGB on binary transparency; this is the default behavior. */
	OnlyOnBinaryTransparency,
	/* Always infill RGB to fully transparent white pixels, even for non-binary alpha channels. */
	Always
};

UCLASS(config=Editor, defaultconfig, meta=(DisplayName="Texture Import"), MinimalAPI)
class UTextureImportSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()
	
public:

	UPROPERTY(config, EditAnywhere, Category=VirtualTextures, meta = (
		DisplayName = "Auto Virtual Texturing Size",
		ToolTip = "Automatically enable the 'Virtual Texture Streaming' texture setting for textures larger than or equal to this size. Zero to disable.  This setting will not affect existing textures in the project."))
	int32 AutoVTSize = 4096;

	UPROPERTY(config, EditAnywhere, Category=VirtualTextures, meta = (
		DisplayName = "Auto resize large textures",
		ToolTip = "If not zero, textures over this size that are not VT will be resized to be equal to or less than this.  Treated as pixel count limit for cubes and volumes.  Must equal AutoVTSize if both are non-zero.  This setting will not affect existing textures in the project."))
	int32 AutoLimitDimension = 0;
	
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta = (
		DisplayName = "Turn on NormalizeNormals for normal maps",
		ToolTip = "NormalizeNormals makes more correct normals in mip maps; it is recommended, but can be turned off to maintain legacy behavior. This setting is applied to newly imported textures, it does not affect existing textures in the project."))
	bool bEnableNormalizeNormals = true;
	
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta = (
		DisplayName = "Turn on fast mip generation filter",
		ToolTip = "Use the fast mip filter on new textures; it is recommended, but can be turned off to maintain legacy behavior. This setting is applied to newly imported textures, it does not affect existing textures in the project."))
	bool bEnableFastMipFilter = true;
	
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta = (
		DisplayName = "CompressionFormat to use for new float textures",
		ToolTip = "Optionally use HDRCompressed (BC6H), or 32-bit adaptively, instead of HDR (RGBA16F) for floating point textures.  This setting is applied to newly imported textures, it does not affect existing textures in the project."))
	ETextureImportFloatingPointFormat CompressedFormatForFloatTextures = ETextureImportFloatingPointFormat::PreviousDefault;
	
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta = (
		DisplayName = "When to infill RGB in transparent white PNG",
		ToolTip = "Default behavior is to infill only for binary transparency; this setting may change that to always or never.  Will check TextureImporter/FillPNGZeroAlpha if this is not changed from Default.  This setting is applied to newly imported textures, it does not affect existing textures in the project. This setting is project-global, prefer the per-user variant in Editor Preferences."))
	ETextureImportPNGInfill PNGInfill = ETextureImportPNGInfill::Default;
	
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta = (
		DisplayName = "Detect nonpow2 textures and set defaults",
		ToolTip = "Sets them to NoMipMaps and uncompressed UI compression. If this is off, artists must imported UI textures up correctly by hand."))
	bool bDoAutomaticTextureSettingsForNonPow2Textures = true;

	//~ Begin UObject Interface

	TEXTUREUTILITIESCOMMON_API virtual void PostInitProperties() override;
	
	// Get the PNGInfill setting, with Default mapped to a concrete choice
	TEXTUREUTILITIESCOMMON_API ETextureImportPNGInfill GetPNGInfillMapDefault() const;

	// get AutoLimitDimension as a pixel count (squared), and with sanity checks applied
	//	zero means do not limit
	TEXTUREUTILITIESCOMMON_API int64 GetAutoLimitPixelCount() const;
	
	// IsImportAutoVTEnabled checks not just this config but also the project VT settings
	TEXTUREUTILITIESCOMMON_API bool IsImportAutoVTEnabled() const;

#if WITH_EDITOR
	TEXTUREUTILITIESCOMMON_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	//~ End UObject Interface
};


class UTexture;

namespace UE::TextureUtilitiesCommon
{
#if WITH_EDITOR
	/* Set default properties on Texture for newly imported textures, or reimports.
	*  Should be called after all texture properties are set, before PostEditChange() 
	*/
	TEXTUREUTILITIESCOMMON_API void ApplyDefaultsForNewlyImportedTextures(UTexture * Texture, bool bIsReimport);
	
	/* Query if texture will be made VT automatically on import due to exceeding AutoVTSize 
		or other conditions which require it to be VT
		only returns true if VT is enabled and auto-VT is enabled
	*/
	TEXTUREUTILITIESCOMMON_API bool ShouldTextureBeVirtualByAutoImportSize(const UTexture * Texture);
#endif

	/* Get the default value for Texture->SRGB
	* ImportImageSRGB is the SRGB setting of the imported image
	*/
	TEXTUREUTILITIESCOMMON_API bool GetDefaultSRGB(TextureCompressionSettings TC, ETextureSourceFormat ImportImageFormat, bool ImportImageSRGB); 
}
