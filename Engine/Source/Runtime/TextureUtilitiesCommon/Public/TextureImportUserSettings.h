// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/TextureDefines.h"
#include "TextureImportSettings.h"

#include "TextureImportUserSettings.generated.h"

struct FPropertyChangedEvent;

UCLASS(config = EditorPerProjectUserSettings, meta=(DisplayName="Texture Import"), MinimalAPI)
class UTextureImportUserSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, Category=ImportSettings, meta = (
		DisplayName = "When to infill RGB in transparent white PNG",
		ToolTip = "Whether to perform infill only for binary transparency, always, or never. If set to 'default', uses global project setting."))
	ETextureImportPNGInfill PNGInfill = ETextureImportPNGInfill::Default;
};

namespace UE::TextureUtilitiesCommon
{
	/** Resolves PNG infill setting using, in order of preference, per-project user settings,
	 * project settings, and legacy config settings.
	 */
	TEXTUREUTILITIESCOMMON_API ETextureImportPNGInfill GetPNGInfillSetting();
}