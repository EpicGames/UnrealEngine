// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/TextureDefines.h"

namespace UE::TextureUtilitiesCommon
{
	/**
	 * Detect the existence of gray scale image in some formats and convert those to a gray scale equivalent image
	 *
	 * @return true if the image was converted
	 */
	template<typename ImageClassType>
	TEXTUREUTILITIESCOMMON_API bool AutoDetectAndChangeGrayScale(ImageClassType& Image);

	/**
	 * For PNG texture importing, this ensures that any pixels with an alpha value of zero have an RGB
	 * assigned to them from a neighboring pixel which has non-zero alpha.
	 * This is needed as PNG exporters tend to turn pixels that are RGBA = (x,x,x,0) to (1,1,1,0)
	 * and this produces artifacts when drawing the texture with bilinear filtering. 
	 *
	 * @param TextureSource - The source texture
	 * @param SourceData - The source texture data
	 */
	TEXTUREUTILITIESCOMMON_API void FillZeroAlphaPNGData(int32 SizeX, int32 SizeY, ETextureSourceFormat SourceFormat, uint8* SourceData, bool bDoOnComplexAlphaNotJustBinaryTransparency);
	
	/**
		* Tests if the given height and width specify a supported texture resolution to import; Can optionally check if the height/width are powers of two
		*
		* @param Width The width of an imported texture whose validity should be checked
		* @param Height The height of an imported texture whose validity should be checked
		* @param bAllowNonPowerOfTwo Whether or not non-power-of-two textures are allowed
		* @param OutErrorMessage Optional output for an error message
		*
		* @return bool true if the given height/width represent a supported texture resolution, false if not
		*
		* NOTE: may open a dialog box to ask user if large non-VT import is wanted!
		*/
	TEXTUREUTILITIESCOMMON_API bool IsImportResolutionValid(int64 Width, int64 Height, bool bAllowNonPowerOfTwo, FText* OutErrorMessage);
}