// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreTypes.h"
#include "Engine/TextureDefines.h"

class UTexture;
class ITargetPlatform;

// Not tested against all the texture types and the api might change or be removed without deprecation
namespace UE::TextureUtilitiesCommon::Experimental
{
	// resize texture source to be <= TargetSourceSize x TargetSourceSize
	//	Halving (mip) steps are done
	//	(only Texture2D and TextureCube are supported at the moment)
	// Note it does not trigger the post edit change after modifying the source texture
	TEXTUREUTILITIESCOMMON_API bool DownsizeTextureSourceData(UTexture* Texture, int32 TargetSourceSize, const ITargetPlatform* TargetPlatform);
	
	// Try to resize the texture source data to a power of two
	// Note it does not trigger the post edit change after modifying the source texture
	TEXTUREUTILITIESCOMMON_API bool ResizeTextureSourceDataToNearestPowerOfTwo(UTexture* Texture);

	// Try to resize the texture source data so that it is not larger than what is built for the specified platform
	//	eg. if the output build size is 1024 but the source size is 2048, the source will be reduced to 1024
	//	Texture->LODBias is changed so that the output built size stays the same
	//	(only Texture2D and TextureCube are supported at the moment)
	//	AdditionalSourceSizeLimit can be used to apply an extra size limit on the source
	//Note: This function DOES trigger the post edit change after modifying the source texture
	TEXTUREUTILITIESCOMMON_API bool DownsizeTextureSourceDataNearRenderingSize(UTexture* Texture, const ITargetPlatform* TargetPlatform, int32 AdditionalSourceSizeLimit = 16384);
	
	// ChangeTextureSourceFormat calls Pre/Post edit change
	//	beware that changing format may change the interpretation of the SRGB bool in Texture
	// ChangeTextureSourceFormat supports mips and blocks (udim) but not layers
	TEXTUREUTILITIESCOMMON_API bool ChangeTextureSourceFormat(UTexture* Texture, ETextureSourceFormat NewFormat);
	
	// Replace TextureSource with JPEG compressed data
	// returns true if change was made
	// calls Pre/PostEditChange :
	//	Quality == 0 for default (85)
	TEXTUREUTILITIESCOMMON_API bool CompressTextureSourceWithJPEG(UTexture* Texture,int32 Quality = 0);
}

#endif // WITH_EDITOR