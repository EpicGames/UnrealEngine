// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

class FString;

namespace UE
{
	namespace TextureUtilitiesCommon
	{
		// UDIM = 1001 + X + Y*10
		//	note the tens digit is not Y ; eg. 1010 is X=9,Y=0

		constexpr int32 UDIM_Max_X = 9;
		constexpr int32 UDIM_Max_Y = 90;
		constexpr int32 UDIM_Min_Index = 1001;
		constexpr int32 UDIM_Max_Index = UDIM_Min_Index + UDIM_Max_X + UDIM_Max_Y * 10;

		// this Regex is unnecessarily complex and broad
		//	UDIM must start with "10" then two digits, or "1100"
		constexpr const TCHAR* DefaultUdimRegexPattern = TEXT(R"((.+?)[._](\d{4})$)");

		TEXTUREUTILITIESCOMMON_API uint32 ParseUDIMName(const FString& Name, const FString& UdimRegexPattern, FString& OutPrefixName, FString& OutPostfixName);

		TEXTUREUTILITIESCOMMON_API int32 GetUDIMIndex(int32 BlockX, int32 BlockY);

		/**
		 * Parse the file for the UDIM pattern
		 * If the pattern is found search in the folder of the file for the others blocks
		 */
		TEXTUREUTILITIESCOMMON_API TMap<int32, FString> GetUDIMBlocksFromSourceFile(const FString& File, const FString& UdimRegexPattern, FString* OutFilenameWithoutUdimPatternAndExtension = nullptr);

		TEXTUREUTILITIESCOMMON_API void ExtractUDIMCoordinates(int32 UDIMIndex, int32& OutBlockX, int32& OutBlockY);
	}
}
