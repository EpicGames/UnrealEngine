// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureUtilitiesCommon : ModuleRules
{
	public TextureUtilitiesCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"DeveloperSettings",
				"Engine",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"ImageCore",
				"Slate",
				"SlateCore",
			});

		// TextureUtilitiesCommon is *incorrectly* in /Runtime/
		// "TextureBuildUtilities" is in /Developer/
		// TextureUtilitiesCommon is not allowed to depend on TextureBuildUtilities

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("TextureBuildUtilities");
		}
	}
}
